{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Sample Tools Editor Mode",
	"Description": "Sample Tools Mode includes a set of sample Tools demonstrating capabilities of the Interactive Tools Framework",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "SampleToolsEditorMode",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": []
}