// Copyright Epic Games, Inc. All Rights Reserved.

#include "CreateActorSampleTool.h"
#include "InteractiveToolManager.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "ToolContextInterfaces.h"

// localization namespace
#define LOCTEXT_NAMESPACE "UCreateActorSampleTool"

/*
 * ToolBuilder
 */


UInteractiveTool* UCreateActorSampleToolBuilder::BuildTool(const FToolBuilderState & SceneState) const
{
	UCreateActorSampleTool* NewTool = NewObject<UCreateActorSampleTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);
	return NewTool;
}



/*
 * Tool
 */


UCreateActorSampleToolProperties::UCreateActorSampleToolProperties()
{
	PlaceOnObjects = true;
	GroundHeight = 0.0f;
}


UCreateActorSampleTool::UCreateActorSampleTool()
{
}


void UCreateActorSampleTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

void UCreateActorSampleTool::Setup()
{
	USingleClickTool::Setup();

	Properties = NewObject<UCreateActorSampleToolProperties>(this);
	AddToolPropertySource(Properties);
}




void UCreateActorSampleTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	// we will create actor at this position
	FVector NewActorPos = FVector::ZeroVector;

	// cast ray into world to find hit position
	FVector RayStart = ClickPos.WorldRay.Origin;
	FVector RayEnd = ClickPos.WorldRay.PointAt(999999);
	FCollisionObjectQueryParams QueryParams(FCollisionObjectQueryParams::AllObjects);
	FHitResult Result;
	bool bHitWorld = TargetWorld->LineTraceSingleByObjectType(Result, RayStart, RayEnd, QueryParams);
	if (Properties->PlaceOnObjects && bHitWorld)
	{
		NewActorPos = Result.ImpactPoint;
	}
	else
	{
		// hit the ground plane
		FPlane GroundPlane(FVector(0,0,Properties->GroundHeight) , FVector(0, 0, 1));
		NewActorPos = FMath::RayPlaneIntersection(ClickPos.WorldRay.Origin, ClickPos.WorldRay.Direction, GroundPlane);
	}

	// create new actor
	FRotator Rotation(0.0f, 0.0f, 0.0f);
	FActorSpawnParameters SpawnInfo;
	AActor* NewActor = TargetWorld->SpawnActor<AActor>(FVector::ZeroVector, Rotation, SpawnInfo);

	// create root component
	USceneComponent* RootComponent = NewObject<USceneComponent>(NewActor, USceneComponent::GetDefaultSceneRootVariableName(), RF_Transactional);
	RootComponent->Mobility = EComponentMobility::Movable;
	RootComponent->bVisualizeComponent = true;
	RootComponent->SetWorldTransform(FTransform(NewActorPos));

	NewActor->SetRootComponent(RootComponent);
	NewActor->AddInstanceComponent(RootComponent);
	RootComponent->RegisterComponent();

}





#undef LOCTEXT_NAMESPACE
