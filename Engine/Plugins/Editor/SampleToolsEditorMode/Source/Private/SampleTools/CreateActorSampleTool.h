// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/SingleClickTool.h"
#include "CreateActorSampleTool.generated.h"


/**
 * Builder for UCreateActorSampleTool
 */
UCLASS()
class SAMPLETOOLSEDITORMODE_API UCreateActorSampleToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override { return true; }
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};



/**
 * Settings UObject for UCreateActorSampleTool. This UClass inherits from UInteractiveToolPropertySet,
 * which provides an OnModified delegate that the Tool will listen to for changes in property values.
 */
UCLASS(Transient)
class SAMPLETOOLSEDITORMODE_API UCreateActorSampleToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UCreateActorSampleToolProperties();

	/** Place actors on existing objects */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Place On Objects"))
	bool PlaceOnObjects;

	/** Height of ground plane */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Ground Height", UIMin = "-1000.0", UIMax = "1000.0", ClampMin = "-1000000", ClampMax = "1000000.0"))
	float GroundHeight;
};




/**
 * UCreateActorSampleTool is an example Tool that drops an empty Actor at each position the user 
 * clicks left mouse button. The Actors are placed at the first ray intersection in the scene,
 * or on a ground plane if no scene objects are hit. All the action is in the ::OnClicked handler.
 */
UCLASS()
class SAMPLETOOLSEDITORMODE_API UCreateActorSampleTool : public USingleClickTool
{
	GENERATED_BODY()

public:
	UCreateActorSampleTool();

	virtual void SetWorld(UWorld* World);

	virtual void Setup() override;

	virtual void OnClicked(const FInputDeviceRay& ClickPos);


protected:
	UPROPERTY()
	TObjectPtr<UCreateActorSampleToolProperties> Properties;


protected:
	/** target World we will raycast into and create Actor in */
	UWorld* TargetWorld;
};
