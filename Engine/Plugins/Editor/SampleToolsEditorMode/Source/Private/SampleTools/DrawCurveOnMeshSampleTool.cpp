// Copyright Epic Games, Inc. All Rights Reserved.

#include "DrawCurveOnMeshSampleTool.h"
#include "Engine/HitResult.h"
#include "InteractiveToolManager.h"
#include "PrimitiveDrawingUtils.h"   // FPrimitiveDrawInterface
#include "ToolContextInterfaces.h"

// localization namespace
#define LOCTEXT_NAMESPACE "UDrawCurveOnMeshSampleTool"

/*
 * ToolBuilder
 */


UMeshSurfacePointTool* UDrawCurveOnMeshSampleToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UDrawCurveOnMeshSampleTool* NewTool = NewObject<UDrawCurveOnMeshSampleTool>(SceneState.ToolManager);
	return NewTool;
}



/*
 * Tool
 */

UDrawCurveOnMeshSampleToolProperties::UDrawCurveOnMeshSampleToolProperties()
{
	Thickness = 4.0f;
	DepthBias = 0.0f;
	MinSpacing = 1.0;
	NormalOffset = 0.25;
	Color = FLinearColor(255, 0, 0);
	bScreenSpace = true;
}


UDrawCurveOnMeshSampleTool::UDrawCurveOnMeshSampleTool()
{
}


void UDrawCurveOnMeshSampleTool::Setup()
{
	UMeshSurfacePointTool::Setup();

	// add settings object
	Settings = NewObject<UDrawCurveOnMeshSampleToolProperties>(this, TEXT("Settings"));
	AddToolPropertySource(Settings);
}



void UDrawCurveOnMeshSampleTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
	
	int NumPts = Positions.Num();
	for (int i = 0; i < NumPts - 1; ++i)
	{
		FVector A = Positions[i] + Settings->NormalOffset*Normals[i];
		FVector B = Positions[i+1] + Settings->NormalOffset*Normals[i+1];
		PDI->DrawLine(A, B, Settings->Color, 0, Settings->Thickness, Settings->DepthBias, Settings->bScreenSpace);
	}
}




void UDrawCurveOnMeshSampleTool::OnBeginDrag(const FRay& Ray)
{
	Positions.Reset();
	Normals.Reset();
	
	FHitResult OutHit;
	if (HitTest(Ray, OutHit))
	{
		Positions.Add(OutHit.ImpactPoint);
		Normals.Add(OutHit.ImpactNormal);
	}

}

void UDrawCurveOnMeshSampleTool::OnUpdateDrag(const FRay& Ray)
{
	FHitResult OutHit;
	if (HitTest(Ray, OutHit))
	{
		if ( FVector::Dist(OutHit.ImpactPoint, Positions[Positions.Num()-1]) > Settings->MinSpacing)
		{
			Positions.Add(OutHit.ImpactPoint);
			Normals.Add(OutHit.ImpactNormal);
		}
	}
}

void UDrawCurveOnMeshSampleTool::OnEndDrag(const FRay& Ray)
{

}

void UDrawCurveOnMeshSampleTool::OnCancelDrag()
{

}




#undef LOCTEXT_NAMESPACE
