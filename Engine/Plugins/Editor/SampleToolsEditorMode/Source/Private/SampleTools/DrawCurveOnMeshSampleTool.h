// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/MeshSurfacePointTool.h"
#include "DrawCurveOnMeshSampleTool.generated.h"


/**
 * UMeshSurfacePointToolBuilder override for UDrawCurveOnMeshSampleTool
 */
UCLASS(Transient)
class SAMPLETOOLSEDITORMODE_API UDrawCurveOnMeshSampleToolBuilder : public UMeshSurfacePointToolBuilder
{
	GENERATED_BODY()

public:
	virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};


/**
 * Settings UObject for UDrawCurveOnMeshSampleTool. This UClass inherits from UInteractiveToolPropertySet,
 * which provides an OnModified delegate that the Tool will listen to for changes in property values.
 */
UCLASS(Transient)
class SAMPLETOOLSEDITORMODE_API UDrawCurveOnMeshSampleToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UDrawCurveOnMeshSampleToolProperties();

	UPROPERTY(EditAnywhere, Category = Options)
	FLinearColor Color;

	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Thickness", UIMin = "0.25", UIMax = "10.0", ClampMin = "0.01", ClampMax = "1000.0"))
	float Thickness;

	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Min Spacing", UIMin = "0.01", UIMax = "10.0"))
	float MinSpacing;

	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Offset", UIMin = "0.0", UIMax = "10.0", ClampMin = "-1000.0", ClampMax = "1000.0"))
	float NormalOffset;

	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Depth Bias", UIMin = "-10.0", UIMax = "10.0"))
	float DepthBias;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bScreenSpace;
};



/**
 * UDrawCurveOnMeshSampleTool is a sample Tool that allows the user to draw curves on the surface of
 * a selected Mesh Component. The various rendering properties of the polycurve are exposed and can be tweaked.
 * Nothing is done with the curve, it is just drawn by ::Render() and discarded when the Tool exits.
 */
UCLASS(Transient)
class SAMPLETOOLSEDITORMODE_API UDrawCurveOnMeshSampleTool : public UMeshSurfacePointTool
{
	GENERATED_BODY()

public:
	UDrawCurveOnMeshSampleTool();

	// UInteractiveTool API

	virtual void Setup() override;

	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	// UMeshSurfacePointTool API
	virtual void OnBeginDrag(const FRay& Ray) override;
	virtual void OnUpdateDrag(const FRay& Ray) override;
	virtual void OnEndDrag(const FRay& Ray) override;
	virtual void OnCancelDrag() override;

protected:
	UPROPERTY()
	TObjectPtr<UDrawCurveOnMeshSampleToolProperties> Settings;

	UPROPERTY()
	TArray<FVector> Positions;

	UPROPERTY()
	TArray<FVector> Normals;

};
