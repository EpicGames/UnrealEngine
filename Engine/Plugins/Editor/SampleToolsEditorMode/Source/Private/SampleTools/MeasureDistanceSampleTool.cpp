// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeasureDistanceSampleTool.h"
#include "InteractiveToolManager.h"
#include "BaseBehaviors/ClickDragBehavior.h"

// for raycast into World
#include "Engine/HitResult.h"
#include "Engine/World.h"

#include "PrimitiveDrawingUtils.h"
#include "ToolContextInterfaces.h"

// localization namespace
#define LOCTEXT_NAMESPACE "UMeasureDistanceSampleTool"

/*
 * ToolBuilder
 */


bool UMeasureDistanceSampleToolBuilder::CanBuildTool(const FToolBuilderState & SceneState) const
{
	return true;
}

UInteractiveTool* UMeasureDistanceSampleToolBuilder::BuildTool(const FToolBuilderState & SceneState) const
{
	UMeasureDistanceSampleTool* NewTool = NewObject<UMeasureDistanceSampleTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);
	return NewTool;
}



/*
 * Tool
 */

UMeasureDistanceProperties::UMeasureDistanceProperties()
{
	// initialize the points and distance to reasonable values
	StartPoint = FVector(0,0,0);
	EndPoint = FVector(0,0,100);
	Distance = 100;
}


void UMeasureDistanceSampleTool::SetWorld(UWorld* World)
{
	check(World);
	this->TargetWorld = World;
}




void UMeasureDistanceSampleTool::Setup()
{
	UInteractiveTool::Setup();

	// Add default mouse input behavior
	UClickDragInputBehavior* MouseBehavior = NewObject<UClickDragInputBehavior>();
	// We will use the shift key to indicate that we should move the second point. 
	// This call tells the Behavior to call our OnUpdateModifierState() function on mouse-down and mouse-move
	MouseBehavior->Modifiers.RegisterModifier(MoveSecondPointModifierID, FInputDeviceState::IsShiftKeyDown);
	MouseBehavior->Initialize(this);
	AddInputBehavior(MouseBehavior);

	// Create the property set and register it with the Tool
	Properties = NewObject<UMeasureDistanceProperties>(this, "Measurement");
	AddToolPropertySource(Properties);
	
	bSecondPointModifierDown = false;
	bMoveSecondPoint = false;
}


void UMeasureDistanceSampleTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	// keep track of the "second point" modifier (shift key for mouse input)
	if (ModifierID == MoveSecondPointModifierID)
	{
		bSecondPointModifierDown = bIsOn;
	}
}



FInputRayHit UMeasureDistanceSampleTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	// we only start drag if press-down is on top of something we can raycast
	FVector Temp;
	FInputRayHit Result = FindRayHit(PressPos.WorldRay, Temp);
	return Result;
}

void UMeasureDistanceSampleTool::OnClickPress(const FInputDeviceRay& PressPos)
{
	// determine whether we are moving first or second point for the drag sequence
	bMoveSecondPoint = bSecondPointModifierDown;
	UpdatePosition(PressPos.WorldRay);
}

void UMeasureDistanceSampleTool::OnClickDrag(const FInputDeviceRay& DragPos)
{
	UpdatePosition(DragPos.WorldRay);
}



FInputRayHit UMeasureDistanceSampleTool::FindRayHit(const FRay& WorldRay, FVector& HitPos)
{
	// trace a ray into the World
	FCollisionObjectQueryParams QueryParams(FCollisionObjectQueryParams::AllObjects);
	FHitResult Result;
	bool bHitWorld = TargetWorld->LineTraceSingleByObjectType(Result, WorldRay.Origin, WorldRay.PointAt(999999), QueryParams);
	if (bHitWorld)
	{
		HitPos = Result.ImpactPoint;
		return FInputRayHit(Result.Distance);
	}
	return FInputRayHit();
}


void UMeasureDistanceSampleTool::UpdatePosition(const FRay& WorldRay)
{
	FInputRayHit HitResult = FindRayHit(WorldRay, (bMoveSecondPoint) ? Properties->EndPoint : Properties->StartPoint);
	if (HitResult.bHit)
	{
		UpdateDistance();
	}
}


void UMeasureDistanceSampleTool::UpdateDistance()
{
	Properties->Distance = FVector::Distance(Properties->StartPoint, Properties->EndPoint);
}


void UMeasureDistanceSampleTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	// if the user updated any of the property fields, update the distance
	UpdateDistance();
}



void UMeasureDistanceSampleTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
	// draw a thin line that shows through objects
	PDI->DrawLine(Properties->StartPoint, Properties->EndPoint,
		FColor(240, 16, 16), SDPG_Foreground, 2.0f, 0.0f, true);
	// draw a thicker line that is depth-tested
	PDI->DrawLine(Properties->StartPoint, Properties->EndPoint,
		FColor(240, 16, 16), SDPG_World, 4.0f, 0.0f, true);
}



#undef LOCTEXT_NAMESPACE
