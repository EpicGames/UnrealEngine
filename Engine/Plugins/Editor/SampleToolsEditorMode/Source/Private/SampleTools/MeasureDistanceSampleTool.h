// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveTool.h"
#include "MeasureDistanceSampleTool.generated.h"


/**
 * Builder for UMeasureDistanceSampleTool
 */
UCLASS()
class SAMPLETOOLSEDITORMODE_API UMeasureDistanceSampleToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};


/**
 * Property set for the UMeasureDistanceSampleTool
 */
UCLASS(Transient)
class SAMPLETOOLSEDITORMODE_API UMeasureDistanceProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UMeasureDistanceProperties();

	/** First point of measurement */
	UPROPERTY(EditAnywhere, Category = Options)
	FVector StartPoint;

	/** Second point of measurement */
	UPROPERTY(EditAnywhere, Category = Options)
	FVector EndPoint;
	
	/** Current distance measurement */
	UPROPERTY(EditAnywhere, Category = Options)
	float Distance;
};



/**
 * UMeasureDistanceSampleTool is an example Tool that allows the user to measure the 
 * distance between two points. The first point is set by click-dragging the mouse, and
 * the second point is set by shift-click-dragging the mouse.
 */
UCLASS()
class SAMPLETOOLSEDITORMODE_API UMeasureDistanceSampleTool : public UInteractiveTool, public IClickDragBehaviorTarget
{
	GENERATED_BODY()

public:
	virtual void SetWorld(UWorld* World);

	// UInteractiveTool overrides

	virtual void Setup() override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IClickDragBehaviorTarget implementation

	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	// these are not used in this Tool
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override {}
	virtual void OnTerminateDragSequence() override {}

	// IModifierToggleBehaviorTarget implementation (inherited via IClickDragBehaviorTarget)

	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;


protected:
	/** Properties of the tool are stored here */
	UPROPERTY()
	TObjectPtr<UMeasureDistanceProperties> Properties;


protected:
	UWorld* TargetWorld = nullptr;		// target World we will raycast into

	static const int MoveSecondPointModifierID = 1;		// identifier we associate with the shift key
	bool bSecondPointModifierDown = false;				// flag we use to keep track of modifier state
	bool bMoveSecondPoint = false;						// flag we use to keep track of which point we are moving during a press-drag

	FInputRayHit FindRayHit(const FRay& WorldRay, FVector& HitPos);		// raycasts into World
	void UpdatePosition(const FRay& WorldRay);					// updates first or second point based on raycast
	void UpdateDistance();										// updates distance
};
