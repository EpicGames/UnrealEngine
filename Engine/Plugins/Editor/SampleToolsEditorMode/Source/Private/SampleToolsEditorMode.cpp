// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleToolsEditorMode.h"
#include "SampleToolsEditorModeToolkit.h"
#include "InteractiveToolManager.h"
#include "SampleToolsEditorModeCommands.h"


//////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////// 
// AddYourTool Step 1 - include the header file for your Tools here
//////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////// 
#include "SampleTools/CreateActorSampleTool.h"
#include "SampleTools/DrawCurveOnMeshSampleTool.h"
#include "SampleTools/MeasureDistanceSampleTool.h"

// step 2: register a ToolBuilder in FSampleToolsEditorMode::Enter()


#define LOCTEXT_NAMESPACE "FSampleToolsEditorMode"

const FEditorModeID USampleToolsEditorMode::EM_SampleToolsEditorModeId = TEXT("EM_SampleToolsEditorMode");


USampleToolsEditorMode::USampleToolsEditorMode()
{
	Info = FEditorModeInfo(USampleToolsEditorMode::EM_SampleToolsEditorModeId,
		LOCTEXT("SampleToolsEditorModeName", "SampleToolsEditorMode"),
		FSlateIcon(),
		true);
}


USampleToolsEditorMode::~USampleToolsEditorMode()
{
}


void USampleToolsEditorMode::ActorSelectionChangeNotify()
{
	// @todo support selection change
}

void USampleToolsEditorMode::Enter()
{
	UEdMode::Enter();

	//////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////
	// AddYourTool Step 2 - register the ToolBuilders for your Tools here.
	// The string name you pass to the ToolManager is used to select/activate your ToolBuilder later.
	//////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////// 
	const FSampleToolsEditorModeCommands& SampleToolCommands = FSampleToolsEditorModeCommands::Get();

	auto CreateActorSampleToolBuilder = NewObject< UCreateActorSampleToolBuilder>(this);
	RegisterTool(SampleToolCommands.CreateActorTool, TEXT("CreateActorSampleTool"), CreateActorSampleToolBuilder);

	RegisterTool(SampleToolCommands.DrawCurveOnMeshTool, TEXT("DrawCurveOnMeshSampleTool"), NewObject<UDrawCurveOnMeshSampleToolBuilder>(this));
	RegisterTool(SampleToolCommands.MeasureDistanceTool, TEXT("MeasureDistanceSampleTool"), NewObject<UMeasureDistanceSampleToolBuilder>(this));
	RegisterTool(SampleToolCommands.SurfacePointTool, TEXT("SurfacePointTool"), NewObject<UMeshSurfacePointToolBuilder>(this));

	// active tool type is not relevant here, we just set to default
	GetToolManager()->SelectActiveToolType(EToolSide::Left, TEXT("SurfacePointTool"));
}

void USampleToolsEditorMode::CreateToolkit()
{
	Toolkit = MakeShareable(new FSampleToolsEditorModeToolkit);
}

TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> USampleToolsEditorMode::GetModeCommands() const
{
	return FSampleToolsEditorModeCommands::Get().GetCommands();
}

#undef LOCTEXT_NAMESPACE
