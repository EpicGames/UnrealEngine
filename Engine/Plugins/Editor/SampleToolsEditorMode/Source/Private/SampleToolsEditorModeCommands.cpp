// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleToolsEditorModeCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FSampleToolsEditorMode"

FSampleToolsEditorModeCommands::FSampleToolsEditorModeCommands()
	: TCommands<FSampleToolsEditorModeCommands>("SampleToolsEditorMode",
		NSLOCTEXT("SampleToolsEditorMode", "SampleToolsEditorModeCommands", "Sample Tools Editor Mode"),
		NAME_None,
		FAppStyle::GetAppStyleSetName())
{
}

void FSampleToolsEditorModeCommands::RegisterCommands()
{
	TArray <TSharedPtr<FUICommandInfo>>& ToolCommands = Commands.FindOrAdd(NAME_Default);
	UI_COMMAND(CreateActorTool, "Create Actor on Click", "Create Actor on Click", EUserInterfaceActionType::ToggleButton, FInputChord());
	ToolCommands.Add(CreateActorTool);
	UI_COMMAND(DrawCurveOnMeshTool, "Draw Curve On Mesh", "Draw Curve On Mesh", EUserInterfaceActionType::ToggleButton, FInputChord());
	ToolCommands.Add(DrawCurveOnMeshTool);
	UI_COMMAND(MeasureDistanceTool, "Measure Distance", "Measure Distance", EUserInterfaceActionType::ToggleButton, FInputChord());
	ToolCommands.Add(MeasureDistanceTool);
	UI_COMMAND(SurfacePointTool, "Surface Point Tool", "Surface Point Tool", EUserInterfaceActionType::ToggleButton, FInputChord());
	ToolCommands.Add(SurfacePointTool);
}

TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> FSampleToolsEditorModeCommands::GetCommands()
{
	return FSampleToolsEditorModeCommands::Get().Commands;
}

#undef LOCTEXT_NAMESPACE
