// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleToolsEditorModeModule.h"
#include "Modules/ModuleManager.h"
#include "SampleToolsEditorModeCommands.h"

#define LOCTEXT_NAMESPACE "FSampleToolsEditorModeModule"

void FSampleToolsEditorModeModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	FSampleToolsEditorModeCommands::Register();
}

void FSampleToolsEditorModeModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	FSampleToolsEditorModeCommands::Unregister();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FSampleToolsEditorModeModule, SampleToolsEditorMode)
