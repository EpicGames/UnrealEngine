// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampleToolsEditorModeToolkit.h"






#define LOCTEXT_NAMESPACE "FSampleToolsEditorModeToolkit"

FSampleToolsEditorModeToolkit::FSampleToolsEditorModeToolkit()
{
}

void FSampleToolsEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	FModeToolkit::Init(InitToolkitHost, InOwningMode);
}

void FSampleToolsEditorModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames.Add(NAME_Default);
}


FName FSampleToolsEditorModeToolkit::GetToolkitFName() const
{
	return FName("SampleToolsEditorMode");
}

FText FSampleToolsEditorModeToolkit::GetBaseToolkitName() const
{
	return NSLOCTEXT("SampleToolsEditorModeToolkit", "DisplayName", "SampleToolsEditorMode Tool");
}

#undef LOCTEXT_NAMESPACE
