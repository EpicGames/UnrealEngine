// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tools/UEdMode.h"

#include "SampleToolsEditorMode.generated.h"

class IAssetGenerationAPI;

/**
 * This class provides an example of how to extend a UEdMode to add some simple tools
 * using the InteractiveTools framework. The various UEdMode input event handlers (see UEdMode.h)
 * forward events to a UEdModeInteractiveToolsContext instance, which 
 * has all the logic for interacting with the InputRouter, ToolManager, etc.
 * The functions provided here are the minimum to get started inserting some custom behavior.
 * Take a look at the UEdMode markup for more extensibility options.
 */
UCLASS()
class USampleToolsEditorMode : public UEdMode
{
	GENERATED_BODY()

public:
	const static FEditorModeID EM_SampleToolsEditorModeId;

	USampleToolsEditorMode();
	virtual ~USampleToolsEditorMode();

	////////////////
	// UEdMode interface
	virtual void Enter() override;
	virtual void ActorSelectionChangeNotify() override;
	virtual void CreateToolkit() override;
	virtual TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> GetModeCommands() const override;

	//////////////////
	// End of UEdMode interface
	//////////////////
};
