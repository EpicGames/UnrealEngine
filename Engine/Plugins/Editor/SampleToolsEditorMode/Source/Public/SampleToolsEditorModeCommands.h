// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

class FSampleToolsEditorModeCommands : public TCommands<FSampleToolsEditorModeCommands>
{
public:
	FSampleToolsEditorModeCommands();

	virtual void RegisterCommands() override;
	static TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> GetCommands();

	TSharedPtr<FUICommandInfo> CreateActorTool;
	TSharedPtr<FUICommandInfo> DrawCurveOnMeshTool;
	TSharedPtr<FUICommandInfo> MeasureDistanceTool;
	TSharedPtr<FUICommandInfo> SurfacePointTool;

protected:
	TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> Commands;
};
