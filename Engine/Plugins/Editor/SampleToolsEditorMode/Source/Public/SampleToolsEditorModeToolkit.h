// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/BaseToolkit.h"
#include "InteractiveTool.h"
#include "SampleToolsEditorMode.h"

class IDetailsView;

/**
 * This FModeToolkit just creates a basic UI panel that allows various InteractiveTools to
 * be initialized, and a DetailsView used to show properties of the active Tool.
 */
class FSampleToolsEditorModeToolkit : public FModeToolkit
{
public:
	FSampleToolsEditorModeToolkit();
	
	// FModeToolkit interface 
	virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;
	virtual void GetToolPaletteNames(TArray<FName>& PaletteNames) const override;

	// IToolkit interface 
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
};
